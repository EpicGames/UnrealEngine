// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

BEGIN_DEFINE_SPEC(FAutomationExpectedErrorTest, "TestFramework.ExpectedError", EAutomationTestFlags::EngineFilter | EAutomationTestFlags_ApplicationContextMask)
END_DEFINE_SPEC(FAutomationExpectedErrorTest)
void FAutomationExpectedErrorTest::Define()
{
	Describe("A defined expected error in a test", [this]()
	{

		It("will not add an error with a number of occurrences less than zero", [this]()
		{
			// Suppress error logged when adding entry with invalid occurrence count
			AddExpectedError(TEXT("number of expected occurrences must be >= 0"), EAutomationExpectedErrorFlags::Contains, 1);

			AddExpectedError(TEXT("The two values are not equal"), EAutomationExpectedErrorFlags::Contains, -1);

			TArray<FAutomationExpectedMessage> Errors;
			GetExpectedMessages(Errors, ELogVerbosity::Warning);

			// Only the first expected error should exist in the list.
			TestEqual("Expected Errors Count", Errors.Num(), 1);

			// Add the expected errors to ensure all test conditions pass
			AddError(TEXT("number of expected occurrences must be >= 0"));
			AddError(TEXT("The two values are not equal"));
		});

		It("can fetch an error with a number of occurrences less than zero", [this]()
		{
			AddExpectedError(TEXT("number of expected occurrences must be >= 0"), EAutomationExpectedErrorFlags::Contains, 1);

			AddExpectedError(TEXT("The two values are not equal"), EAutomationExpectedErrorFlags::Contains, -1);

			const bool bIncludeSuppressed = true;
			TArray<FAutomationExpectedMessage> Errors;
			GetExpectedMessages(Errors, bIncludeSuppressed, ELogVerbosity::Warning);

			// The suppressed error should show up in the list.
			TestEqual("Expected Errors Count", Errors.Num(), 2);

			// Add the expected errors to ensure all test conditions pass
			AddError(TEXT("number of expected occurrences must be >= 0"));
			AddError(TEXT("The two values are not equal"));
		});

		It("will add an error with a number of occurrences equal to zero", [this]()
		{
			AddExpectedError(TEXT("Expected Error"), EAutomationExpectedErrorFlags::Contains, 0);

			TArray<FAutomationExpectedMessage> Errors;
			GetExpectedMessages(Errors, ELogVerbosity::Warning);
			TestEqual("Expected Errors Count", Errors.Num(), 1);

			// Add the expected error to ensure all test conditions pass
			AddError(TEXT("Expected Error"));
		});

		It("will not duplicate an existing expected error using the same matcher", [this]()
		{
			AddExpectedError(TEXT("Expected Error"), EAutomationExpectedErrorFlags::Contains, 1);
			AddExpectedError(TEXT("Expected Error"), EAutomationExpectedErrorFlags::Contains, 1);

			TArray<FAutomationExpectedMessage> Errors;
			GetExpectedMessages(Errors, ELogVerbosity::Warning);
			TestEqual("Expected Errors Count", Errors.Num(), 1);

			// Add the expected error to ensure all test conditions pass
			AddError(TEXT("Expected Error"));
		});

		It("will not duplicate an expected error using a different matcher", [this]()
		{
			AddExpectedError(TEXT("Expected Exact Error"), EAutomationExpectedErrorFlags::Exact, 1);
			AddExpectedError(TEXT("Expected Exact Error"), EAutomationExpectedErrorFlags::Contains, 1);

			AddExpectedError(TEXT("Expected Contains Error"), EAutomationExpectedErrorFlags::Contains, 1);
			AddExpectedError(TEXT("Expected Contains Error"), EAutomationExpectedErrorFlags::Exact, 1);

			TArray<FAutomationExpectedMessage> Errors;
			GetExpectedMessages(Errors, ELogVerbosity::Warning);
			TestEqual("Expected Errors Count", Errors.Num(), 2);

			// Add the expected errors to ensure all test conditions pass
			AddError(TEXT("Expected Exact Error"));
			AddError(TEXT("Expected Contains Error"));
		});

		// Disabled until fix for UE-44340 (crash creating invalid regex) is merged
		xIt("will not add an error with an invalid regex pattern", [this]()
		{
			AddExpectedError(TEXT("invalid regex }])([{"), EAutomationExpectedErrorFlags::Contains, 0);

			TArray<FAutomationExpectedMessage> Errors;
			GetExpectedMessages(Errors, ELogVerbosity::Warning);

			TestEqual("Expected Errors Count", Errors.Num(), 0);
		});

		It("will match both Error and Warning messages", [this]()
		{
			AddExpectedError(TEXT("Expected Message"), EAutomationExpectedErrorFlags::Contains, 0);
			AddError(TEXT("Expected Message"));
			AddWarning(TEXT("Expected Message"));
		});

		It("will not fail or warn if encountered", [this]()
		{
			AddExpectedError(TEXT("Expected Error"), EAutomationExpectedErrorFlags::Contains, 1);
			AddExpectedError(TEXT("Expected Warning"), EAutomationExpectedErrorFlags::Contains, 1);
			AddError(TEXT("Expected Error"));
			AddWarning(TEXT("Expected Warning"));
		});

		It("will not match multiple occurrences in the same message when using Contains matcher", [this]()
		{
			AddExpectedError(TEXT("Expected"), EAutomationExpectedErrorFlags::Contains, 1);
			AddError(TEXT("ExpectedExpectedExpected ExpectedExpectedExpected"));
		});

		It("will match different messages that fit the regex pattern", [this]()
		{
			AddExpectedError(TEXT("Response \\(-?\\d+\\)"), EAutomationExpectedErrorFlags::Contains, 4);
			AddError(TEXT("Response (0)"));
			AddError(TEXT("Response (1)"));
			AddError(FString::Printf(TEXT("Response (%lld)"), MIN_int64));
			AddError(FString::Printf(TEXT("Response (%lld)"), MAX_uint64));
		});

		It("will match a regex pattern as case insensitive", [this]()
		{
			AddExpectedError(TEXT("Expecte. Error"), EAutomationExpectedErrorFlags::Contains, 0);
			AddError(TEXT("EXPECTED ERROR"));
		});

		It("will match a message that contains the plain string pattern", [this]()
		{
			AddExpectedErrorPlain(TEXT("Expected Error"), EAutomationExpectedErrorFlags::Contains, 1);
			AddError(TEXT("Some Expected Error not catched"));
		});

		It("will match a message as case insensitive", [this]()
		{
			AddExpectedErrorPlain(TEXT("Expected Error"), EAutomationExpectedErrorFlags::Contains, 1);
			AddError(TEXT("Some EXPECTED ERROR not catched"));
		});

		It("will match a message that is the exact match of the plain string pattern", [this]()
		{
			AddExpectedErrorPlain(TEXT("Expected Error not catched"), EAutomationExpectedErrorFlags::Exact, 1);
			AddError(TEXT("Expected Error not catched"));
		});
	});

}

// Tests for cases where expected errors will fail a test.
// IMPORTANT: The pass condition for these tests is that they FAIL. To prevent
// the expected failures from interfering with regular test runs, these tests
// must be run manually.
BEGIN_DEFINE_SPEC(FAutomationExpectedErrorFailureTest, "TestFramework.ExpectedError", EAutomationTestFlags::NegativeFilter | EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::RequiresUser)
END_DEFINE_SPEC(FAutomationExpectedErrorFailureTest)
void FAutomationExpectedErrorFailureTest::Define()
{
	Describe("An expected error failure", [this]()
	{
		It("will occur if expected a specific number of times and NOT encountered.", [this]()
		{
			AddExpectedError(TEXT("Expected Error"), EAutomationExpectedErrorFlags::Exact, 1);
		});

		It("will occur if expected a specific number of times and is encountered too few times.", [this]()
		{
			AddExpectedError(TEXT("Expected Error"), EAutomationExpectedErrorFlags::Exact, 2);
			AddError(TEXT("Expected Error"));
		});

		It("will occur if expected a specific number of times and is encountered too many times.", [this]()
		{
			AddExpectedError(TEXT("Expected Error"), EAutomationExpectedErrorFlags::Exact, 1);
			AddError(TEXT("Expected Error"));
			AddError(TEXT("Expected Error"));
		});

		It("will occur if expected any number of times and is not encountered.", [this]()
		{
			AddExpectedError(TEXT("Expected Error"), EAutomationExpectedErrorFlags::Exact, 0);
		});

		It("will occur if multiple expected errors are NOT encountered.", [this]()
		{
			AddExpectedError(TEXT("Expected Error 1"), EAutomationExpectedErrorFlags::Exact, 1);
			AddExpectedError(TEXT("Expected Error 2"), EAutomationExpectedErrorFlags::Contains, 1);
		});

		It("will occur if not all expected errors are encountered.", [this]()
		{
			AddExpectedError(TEXT("Expected error 1"), EAutomationExpectedErrorFlags::Exact, 1);
			AddExpectedError(TEXT("Expected error 2"), EAutomationExpectedErrorFlags::Contains, 1);
			AddError(TEXT("Expected error 1"));
		});

		It("will occur if only partial matches are encountered when using Exact matcher", [this]()
		{
			AddExpectedError(TEXT("Expected"), EAutomationExpectedErrorFlags::Exact, 1);
			AddError(TEXT("Expected error"));
		});

		It("will occur if a message that is not the exact match of the plain string pattern is encountered", [this]()
		{
			AddExpectedErrorPlain(TEXT("Expected Error"), EAutomationExpectedErrorFlags::Exact, 0);
			AddError(TEXT("Some Expected Error not catched"));
		});

		It("will occur if a message that is a regex match pattern is encountered", [this]()
		{
			AddExpectedErrorPlain(TEXT("Expecte. Error"), EAutomationExpectedErrorFlags::Contains, 0);
			AddError(TEXT("Some Expected Error not catched"));
		});
	});
}

BEGIN_DEFINE_SPEC(FAutomationExpectedMessageTest, "TestFramework.ExpectedMessage", EAutomationTestFlags::EngineFilter | EAutomationTestFlags_ApplicationContextMask)
END_DEFINE_SPEC(FAutomationExpectedMessageTest)
void FAutomationExpectedMessageTest::Define()
{
	Describe("A defined expected message with a specified verbosity ", [this]()
	{

		It("will include error and fatal categories when checking for warning messages", [this]()
		{
			AddExpectedMessage(TEXT("suppress this error message"), ELogVerbosity::Error, EAutomationExpectedMessageFlags::Contains, 1);
			AddExpectedMessage(TEXT("suppress this fatal message"), ELogVerbosity::Fatal, EAutomationExpectedMessageFlags::Contains, 1);

			TArray<FAutomationExpectedMessage> Messages;
			GetExpectedMessages(Messages, ELogVerbosity::Warning);

			TestEqual(TEXT("Both the error and fatal messages are returned when Warning is specified as the maximum verbosity"), Messages.Num(), 2);

			TestTrue(TEXT("An Error message also counts as an error"), LogCategoryMatchesSeverityInclusive(Messages[0].Verbosity, ELogVerbosity::Warning));
			TestTrue(TEXT("A Fatal message also counts as an error"), LogCategoryMatchesSeverityInclusive(Messages[1].Verbosity, ELogVerbosity::Warning));

			AddError(TEXT("suppress this error message please"));
			AddError(TEXT("also suppress this fatal message"));
		});


		// An expected message with a verbosity of "all" behaves as "any"
		It("will be included when the message is \"all\" but the queried verbosity is NOT \"all\"", [this]()
		{
			AddExpectedMessage(TEXT("suppress this message"), ELogVerbosity::All, EAutomationExpectedMessageFlags::Contains, 1);

			// Message with verbosity "all" should always be returned regardless of requested verbosity here
			TArray<FAutomationExpectedMessage> Messages;
			GetExpectedMessages(Messages, ELogVerbosity::Warning);

			TestEqual(TEXT("The message with the intention of being any category is returned when something other than \"all\" is specified as the maximum verbosity to query"), Messages.Num(), 1);

			AddInfo(TEXT("suppress this message please"));
		});

	});

}
