// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

BEGIN_DEFINE_SPEC(AutomationSpec, "TestFramework.Spec", EAutomationTestFlags::EngineFilter | EAutomationTestFlags_ApplicationContextMask)
	bool Foo;
	FString RunOrder; 
END_DEFINE_SPEC(AutomationSpec)
void AutomationSpec::Define()
{
	Describe("A Describe 1", [this]()
	{
		It("contains a spec with an expectation", [this]()
		{
			Foo = true;
			TestEqual("Foo", Foo, true);
		});
	});

	Describe("A Describe 2", [this]()
	{
		BeforeEach([&]()
		{
			Foo = false;
		});

		xIt("will not run disabled specs", [this]()
		{
			TestEqual("Foo", Foo, true);
		});

		xDescribe("with disabled nested Describes", [this]()
		{
			It("will not run specs within those Describes", [this]()
			{
				TestEqual("Foo", Foo, true);
			});
		});
	});

	Describe("A spec using BeforeEach and AfterEach", [this]()
	{
		BeforeEach([this]()
		{
			RunOrder = TEXT("A");
		});

		It("will run code before each spec in the Describe and after each spec in the Describe", [this]()
		{
			TestEqual("RunOrder", RunOrder, TEXT("A"));
		});

		AfterEach([this]()
		{
			RunOrder += TEXT("Z");
			TestEqual("RunOrder", RunOrder, TEXT("AZ"));
		});
	});

	Describe("A spec using BeforeEach and AfterEach", [this]()
	{
		AfterEach([this]()
		{
			RunOrder += TEXT("Z");
			TestEqual("RunOrder", RunOrder, TEXT("ABYZ"));
		});

		Describe("while nested inside another Describe", [this]()
		{
			It("will run all BeforeEach blocks and all AfterEach blocks", [this]()
			{
				TestEqual("RunOrder", RunOrder, TEXT("AB"));
			});

			AfterEach([this]()
			{
				RunOrder += TEXT("Y");
			});

			BeforeEach([this]()
			{
				RunOrder += TEXT("B");
			});
		});

		BeforeEach([this]()
		{
			RunOrder = TEXT("A");
		});
	});

	Describe("A spec using BeforeEach and AfterEach", [this]()
	{
		BeforeEach([this]()
		{
			RunOrder = TEXT("A");
		});

		AfterEach([this]()
		{
			RunOrder += TEXT("Z");
			TestEqual("RunOrder", RunOrder, TEXT("ABCDXYZ"));
		});

		BeforeEach([this]()
		{
			RunOrder += TEXT("B");
		});

		Describe("while nested inside another Describe", [this]()
		{
			AfterEach([this]()
			{
				RunOrder += TEXT("Y");
			});

			BeforeEach([this]()
			{
				RunOrder += TEXT("C");
			});

			Describe("while nested inside yet another Describe", [this]()
			{
				It("will run all BeforeEach blocks and all AfterEach blocks", [this]()
				{
					TestEqual("RunOrder", RunOrder, TEXT("ABCD"));
				});

				AfterEach([this]()
				{
					RunOrder += TEXT("X");
				});

				BeforeEach([this]()
				{
					RunOrder += TEXT("D");
				});
			});
		});
	});

	Describe("A spec async", [this]()
	{
		Describe("will run BeforeEach and AfterEach blocks", [this]()
		{
			BeforeEach([this]()
			{
				RunOrder = TEXT("A");
			});

			It("", EAsyncExecution::ThreadPool, [this]()
			{
				RunOrder += TEXT("B");
			});

			AfterEach([this]()
			{
				RunOrder += TEXT("C");
				TestEqual("RunOrder", RunOrder, TEXT("ABC"));
			});
		});
	});

	Describe("LatentIt", [this]()
	{
		Describe("is called only once", [this]()
		{
			BeforeEach([this]()
			{
				RunOrder = TEXT("");
			});

			LatentIt("", [this](const FDoneDelegate Done)
			{
				RunOrder += TEXT("X");
				if (RunOrder == TEXT("X"))
				{
					Async(EAsyncExecution::ThreadPool, [this, Done]() {
						FPlatformProcess::Sleep(0.3f);
						Done.Execute();
					});
				}
			});

			AfterEach([this]()
			{
				TestEqual("RunOrder", RunOrder, TEXT("X"));
			});
		});

		Describe("can early exit", [this]()
		{
			BeforeEach([this]()
			{
				RunOrder = TEXT("");
			});

			LatentIt("", [this](const FDoneDelegate Done)
			{
				RunOrder += TEXT("X");
				if (RunOrder == TEXT("X"))
				{
					Done.Execute();
				}
			});

			AfterEach([this]()
			{
				TestEqual("RunOrder", RunOrder, TEXT("X"));
			});
		});

		Describe("can be run async", [this]()
		{
			BeforeEach([this]()
			{
				RunOrder = TEXT("");
			});

			LatentIt("", EAsyncExecution::ThreadPool, [this](const FDoneDelegate Done)
			{
				FPlatformProcess::Sleep(0.1f);
				RunOrder += TEXT("X");
				Done.Execute();
			});

			AfterEach([this]()
			{
				TestEqual("RunOrder", RunOrder, TEXT("X"));
			});
		});

		Describe("can exit early async", [this]()
		{
			BeforeEach([this]()
			{
				RunOrder = TEXT("");
			});

			LatentIt("", EAsyncExecution::ThreadPool, [this](const FDoneDelegate Done)
			{
				RunOrder += TEXT("X");
				Done.Execute();
			});

			AfterEach([this]()
			{
				TestEqual("RunOrder", RunOrder, TEXT("X"));
			});
		});
	});
}
