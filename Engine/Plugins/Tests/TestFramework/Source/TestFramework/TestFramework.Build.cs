// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TestFramework : ModuleRules
{
	public TestFramework(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				// ... add other public dependencies that you statically link with here ...
			}
			);


		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Engine",
				// ... add private dependencies that you statically link with here ...	
			}
			);
	}
}
